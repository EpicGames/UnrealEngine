{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Platform DLC",
	"Description": "Experimental support for native platform DLC (on supported platforms)",
	"CreatedBy": "Epic Games, Inc.",
	"Category": "Platform",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"RequiresBuildPlatform": false,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "PlatformDLC",
			"Type": "RuntimeNoCommandlet",
			"LoadingPhase": "PostConfigInit"
		}
	]
}