// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using EpicGames.Core;

public class PlatformDLC : ModuleRules
{
	public PlatformDLC(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.Add("Core");
	}
}
