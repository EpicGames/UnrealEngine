// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlatformDLC.h"
#include "PlatformDLCModule.h"
#include "Containers/Ticker.h"
#include "Misc/ScopeLock.h"


bool FGenericPlatformDLC::InitializeAsync()
{
	bool bShouldStart = false;
	{
		FScopeLock Lock(&InitLock);
		if (InitState.load(std::memory_order_relaxed) == EInitState::NotStarted)
		{
			InitState.store(EInitState::InProgress, std::memory_order_relaxed);
			bShouldStart = true;
		}
	}

	if (bShouldStart)
	{
		if (IsInGameThread())
		{
			BeginInitializeInternal();
		}
		else
		{
			ExecuteOnGameThread(UE_SOURCE_LOCATION, [this]()
			{
				BeginInitializeInternal();
			});
		}
		return true;
	}
	return false;
}


void FGenericPlatformDLC::RegisterInitializationCallback(TFunction<void()> Callback)
{
	check(Callback);

	bool bAlreadyComplete;
	{
		FScopeLock Lock(&InitLock);
		bAlreadyComplete = (InitState.load(std::memory_order_relaxed) == EInitState::Complete);
		if (!bAlreadyComplete)
		{
			PendingInitCallbacks.Add(MoveTemp(Callback));
		}
	}

	if (bAlreadyComplete)
	{
		ExecuteOnGameThread(UE_SOURCE_LOCATION, MoveTemp(Callback));
	}
}


void FGenericPlatformDLC::EndInitializeInternal()
{
	TArray<TFunction<void()>> InitCallbacks;
	{
		FScopeLock Lock(&InitLock);
		check(InitState.load(std::memory_order_relaxed) == EInitState::InProgress);

		InitState.store(EInitState::Complete, std::memory_order_release);
		InitCallbacks = MoveTemp(PendingInitCallbacks);
	}

	for (TFunction<void()>& InitCallback : InitCallbacks)
	{
		ExecuteOnGameThread(UE_SOURCE_LOCATION, MoveTemp(InitCallback));
	}

	UE_LOG(LogPlatformDLC, Log, TEXT("Initialization complete"));
}


void FGenericPlatformDLC::Shutdown()
{
	FScopeLock Lock(&InitLock);
	InitState.store(EInitState::NotStarted, std::memory_order_relaxed);
	PendingInitCallbacks.Empty();
}


void FGenericPlatformDLC::PostNotification( FName DLCName, ENotification Notification, bool bSuccess )
{
	UE_LOGF(LogPlatformDLC, Log, "PostNotification %ls: %ls %ls", *DLCName.ToString(), *LexToString(Notification), *LexToString(bSuccess));
	ExecuteOnGameThread( UE_SOURCE_LOCATION, [this, DLCName, Notification, bSuccess]
	{
		NotificationDelegate.Broadcast(DLCName, Notification, bSuccess);
	});
}
