// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlatformDLCModule.h"
#include "PlatformDLCPlatformFile.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/ConfigCacheIni.h"

DEFINE_LOG_CATEGORY(LogPlatformDLC);


const TCHAR* IPlatformDLC::ConfigSectionName = TEXT("PlatformDLC");


class FPlatformDLCModule : public IPlatformDLCModule
{
public:
	virtual void StartupModule() override
	{
		PlatformDLCFile = IPlatformDLCPlatformFile::Construct();
	}

	virtual void ShutdownModule() override
	{
		// remove DLC notification hook
		UnregisterForNotifications();

		// remove ourselves from the platform file chain (there can be late writes after the shutdown).
		UnregisterPlatformFile();
		PlatformDLCFile.Reset();
	}

	virtual TSharedPtr<IPlatformDLC> GetPlatformDLC() override
	{
		if (!bHasTried)
		{
			// read the configuration & try to create the default platform DLC module
			FString DefaultModule;
			if (GConfig->GetString(IPlatformDLC::ConfigSectionName, TEXT("DefaultModule"), DefaultModule, GEngineIni) && !DefaultModule.IsEmpty())
			{
				IPlatformDLCFactoryModule* PlatformDLCFactory = FModuleManager::LoadModulePtr<IPlatformDLCFactoryModule>(*DefaultModule);
				if (PlatformDLCFactory != nullptr)
				{
					PlatformDLC = PlatformDLCFactory->GetPlatformDLC();
					RegisterForNotifications();
				}
			}

			bHasTried = true;
		}

		return PlatformDLC.Pin();
	}

	virtual IPlatformFile* GetPlatformFile() override
	{
		check(PlatformDLCFile.IsValid());
		return PlatformDLCFile.Get();
	}

private:

	void RegisterPlatformFile()
	{
		verify(PlatformDLCFile->Initialize(&FPlatformFileManager::Get().GetPlatformFile(), nullptr));
		FPlatformFileManager::Get().SetPlatformFile(*PlatformDLCFile);
	}

	void UnregisterPlatformFile()
	{
		if (FPlatformFileManager::Get().FindPlatformFile(PlatformDLCFile->GetName()))
		{
			FPlatformFileManager::Get().RemovePlatformFile(PlatformDLCFile.Get());
		}
	}


	void RegisterForNotifications()
	{
		check(!NotificationDelegate.IsValid());

		TSharedPtr<IPlatformDLC> PlatformDLCStrong = PlatformDLC.Pin();
		if (PlatformDLCStrong.IsValid())
		{
			NotificationDelegate = PlatformDLCStrong->OnNotification().AddRaw(this, &FPlatformDLCModule::OnNotification );
		}
	}

	void UnregisterForNotifications()
	{
		TSharedPtr<IPlatformDLC> PlatformDLCStrong = PlatformDLC.Pin();
		if (PlatformDLCStrong.IsValid())
		{
			PlatformDLCStrong->OnNotification().Remove(NotificationDelegate);
		}
		NotificationDelegate.Reset();
	}

	void OnNotification( FName DLCName, IPlatformDLC::ENotification Notification, bool bSuccess )
	{
		check(IsInGameThread());

		if (Notification == IPlatformDLC::ENotification::Mounted && bSuccess)
		{
			TSharedPtr<IPlatformDLC> PlatformDLCStrong = PlatformDLC.Pin();
			if (PlatformDLCStrong.IsValid())
			{
				FString MountPoint = PlatformDLCStrong->GetRootDirectory(DLCName);

				bool bHasMountPoints = PlatformDLCFile->HasMountPoints();
				PlatformDLCFile->AddMountPoint(DLCName, MountPoint);
				if (!bHasMountPoints && PlatformDLCFile->HasMountPoints())
				{
					RegisterPlatformFile();
				}
			}
		}
		else if (Notification == IPlatformDLC::ENotification::Unmounted) // note: ignoring bSuccess here because the game's intent is for the DLC to unmount
		{
			bool bHasMountPoints = PlatformDLCFile->HasMountPoints();
			PlatformDLCFile->RemoveMountPoint(DLCName);
			if (bHasMountPoints && !PlatformDLCFile->HasMountPoints())
			{
				UnregisterPlatformFile();
			}
		}
	};

	bool bHasTried = false;
	TWeakPtr<IPlatformDLC> PlatformDLC;
	TUniquePtr<IPlatformDLCPlatformFile> PlatformDLCFile;
	FDelegateHandle NotificationDelegate;

};



static FAutoConsoleCommand CVarPlatformDLCMount(
	TEXT("PlatformDLC.Mount"),
	TEXT("[DLCName] Mount the given platform DLC"),
	FConsoleCommandWithArgsDelegate::CreateLambda( [](const TArray<FString>& Args )
	{
		TSharedPtr<IPlatformDLC> PlatformDLC = IPlatformDLCModule::Get().GetPlatformDLC();
		if( PlatformDLC.IsValid() && Args.Num() > 0)
		{
			bool bResult = PlatformDLC->Mount(FName(Args[0]));
			UE_LOGF(LogPlatformDLC, Log, "DLC Mount(%ls) returned %ls", *Args[0], *LexToString(bResult) ); 
		}
	})
);

static FAutoConsoleCommand CVarPlatformDLCUnmount(
	TEXT("PlatformDLC.Unmount"),
	TEXT("[DLCName] Unmount the given platform DLC"),
	FConsoleCommandWithArgsDelegate::CreateLambda( [](const TArray<FString>& Args )
	{
		TSharedPtr<IPlatformDLC> PlatformDLC = IPlatformDLCModule::Get().GetPlatformDLC();
		if( PlatformDLC.IsValid() && Args.Num() > 0)
		{
			bool bResult = PlatformDLC->Unmount(FName(Args[0]));
			UE_LOGF(LogPlatformDLC, Log, "DLC Unmount(%ls) returned %ls", *Args[0], *LexToString(bResult) ); 
		}
	})
);

static FAutoConsoleCommand CVarPlatformDLCDownload(
	TEXT("PlatformDLC.Download"),
	TEXT("[DLCName] Download the given DLC"),
	FConsoleCommandWithArgsDelegate::CreateLambda( [](const TArray<FString>& Args )
	{
		TSharedPtr<IPlatformDLC> PlatformDLC = IPlatformDLCModule::Get().GetPlatformDLC();
		if( PlatformDLC.IsValid() && Args.Num() > 0)
		{
			bool bResult = PlatformDLC->Download(FName(Args[0]));
			UE_LOGF(LogPlatformDLC, Log, "DLC Download(%ls) returned %ls", *Args[0], *LexToString(bResult) );
		}
	})
);

static FAutoConsoleCommand CVarPlatformDLCUninstall(
	TEXT("PlatformDLC.Uninstall"),
	TEXT("[DLCName] Uninstalls the given DLC (debug only)"),
	FConsoleCommandWithArgsDelegate::CreateLambda( [](const TArray<FString>& Args )
	{
		TSharedPtr<IPlatformDLC> PlatformDLC = IPlatformDLCModule::Get().GetPlatformDLC();
		if( PlatformDLC.IsValid() && Args.Num() > 0)
		{
			bool bResult = PlatformDLC->Uninstall(FName(Args[0]));
			UE_LOGF(LogPlatformDLC, Log, "Uninstal(%ls) returned %ls", *Args[0], *LexToString(bResult) );
		}
	})
);

static FAutoConsoleCommand CVarPlatformDLCDir(
	TEXT("PlatformDLC.Debug.Dir"),
	TEXT("[DLCName] Dump all files in the given DLC (debug only)"),
	FConsoleCommandWithArgsDelegate::CreateLambda( [](const TArray<FString>& Args )
	{
		TSharedPtr<IPlatformDLC> PlatformDLC = IPlatformDLCModule::Get().GetPlatformDLC();
		if( PlatformDLC.IsValid() && Args.Num() > 0)
		{
			FName DLCName = FName(Args[0]);
			if (PlatformDLC->GetState(DLCName) == IPlatformDLC::EState::Mounted)
			{
				FString MountPath = PlatformDLC->GetRootDirectory(DLCName);
				if (!MountPath.IsEmpty())
				{
					UE_LOGF(LogPlatformDLC, Log, "All Files For DLC '%ls' in Mount Point '%ls'", *Args[0], *MountPath);
	
					class FDumpFileVistor : public IPlatformFile::FDirectoryVisitor
					{
					public:
						virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
						{
							UE_CLOGF(!bIsDirectory, LogPlatformDLC, Log, "%ls", FilenameOrDirectory);
							return true;
						}
					};
					FDumpFileVistor Visitor;
					IPlatformFile::GetPlatformPhysical().IterateDirectoryRecursively(*MountPath, Visitor);
				}
				else
				{
					UE_LOGF(LogPlatformDLC, Error, "DLC '%ls' has no file system mount point", *Args[0]);
				}
			}
			else
			{
				UE_LOGF(LogPlatformDLC, Error, "DLC '%ls' is not mounted", *Args[0]);
			}
		}
	})
);


static FAutoConsoleCommand CVarPlatformDLCList(
	TEXT("PlatformDLC.Debug.List"),
	TEXT("Lists all known DLC names (debug only)"),
	FConsoleCommandDelegate::CreateLambda([]()
	{
		TSharedPtr<IPlatformDLC> PlatformDLC = IPlatformDLCModule::Get().GetPlatformDLC();
		if( PlatformDLC.IsValid())
		{
			TArray<FName> DLCNames = PlatformDLC->GetAllDLCNames();
			UE_LOGF(LogPlatformDLC, Log, "%d DLCs", DLCNames.Num() );
			for (const FName& DLCName : DLCNames)
			{
				UE_LOGF(LogPlatformDLC, Log, "    %-32ls State:%-12ls Entitled:%-5ls StoreId:%ls", *DLCName.ToString(), *LexToString(PlatformDLC->GetState(DLCName)), *LexToString(PlatformDLC->HasEntitlement(DLCName)), *PlatformDLC->GetStoreId(DLCName) );
			}
		}
	})
);




FString LexToString(IPlatformDLC::EState State)
{
	switch (State)
	{
		case IPlatformDLC::EState::NotInstalled:  return TEXT("NotInstalled");
		case IPlatformDLC::EState::Downloading:   return TEXT("Downloading");
		case IPlatformDLC::EState::Downloaded:    return TEXT("Downloaded");
		case IPlatformDLC::EState::Mounting:      return TEXT("Mounting");
		case IPlatformDLC::EState::Mounted:       return TEXT("Mounted");
		case IPlatformDLC::EState::Unmounting:    return TEXT("Unmounting");
		case IPlatformDLC::EState::Uninstalling:  return TEXT("Uninstalling");
		default:                                  return TEXT("Invalid");
	}
}

FString LexToString(IPlatformDLC::ENotification Notification)
{
	switch (Notification)
	{
		case IPlatformDLC::ENotification::Entitlement:  return TEXT("Entitlement");
		case IPlatformDLC::ENotification::Mounted:      return TEXT("Mounted");
		case IPlatformDLC::ENotification::Unmounted:    return TEXT("Unmounted");
		case IPlatformDLC::ENotification::Downloaded:   return TEXT("Downloaded");
		case IPlatformDLC::ENotification::Uninstalled:  return TEXT("Uninstalled");
		default:                                        return TEXT("Invalid");
	}
}


IMPLEMENT_MODULE(FPlatformDLCModule, PlatformDLC);
