// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlatformDLCPlatformFile.h"
#include "PlatformDLCModule.h"
#include "Containers/DirectoryTree.h"
#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "Async/MappedFileHandle.h"
#include "Misc/Paths.h"
#include "Misc/ScopeRWLock.h"

#if !defined(PLATFORMDLC_DUMP_LOOSE_FILES_ON_MOUNT)
#define PLATFORMDLC_DUMP_LOOSE_FILES_ON_MOUNT (UE_BUILD_DEVELOPMENT || UE_BUILD_DEBUG)
#endif

namespace UE::PlatformDLC::Private
{
	/**
	 * A file/directory visitor for files in DLC, used to share code for FDirectoryVisitor and FDirectoryStatVisitor
	 * when iterating over files in DLC.
	 */
	class FDLCFileDirectoryVisitorBase
	{
	public:
		virtual ~FDLCFileDirectoryVisitorBase() = default;
		virtual bool ShouldVisitLeafPathname(FStringView LeafPathname) = 0;
		virtual bool Visit(const FString& Filename, bool bIsDirectory, const FString* AbsPath) = 0;
	};

	/** FDLCFileDirectoryVisitorBase for a FDirectoryVisitor. */
	class FDLCFileDirectoryVisitor : public FDLCFileDirectoryVisitorBase
	{
	public:
		explicit FDLCFileDirectoryVisitor(IPlatformFile::FDirectoryVisitor& InInner)
			: Inner(InInner) {}

		virtual bool ShouldVisitLeafPathname(FStringView LeafPathname) override
		{
			return Inner.ShouldVisitLeafPathname(LeafPathname);
		}
		virtual bool Visit(const FString& Filename, bool bIsDirectory, const FString* AbsPath) override
		{
			return Inner.Visit(*Filename, bIsDirectory);
		}

		IPlatformFile::FDirectoryVisitor& Inner;
	};

	/** FDLCFileDirectoryVisitorBase for a FDirectoryStatVisitor. */
	class FDLCFileDirectoryStatVisitor : public FDLCFileDirectoryVisitorBase
	{
	public:
		FDLCFileDirectoryStatVisitor(IPlatformFile& InLowerLevel, IPlatformFile::FDirectoryStatVisitor& InInner)
			: LowerLevel(InLowerLevel), Inner(InInner) {}

		virtual bool ShouldVisitLeafPathname(FStringView LeafPathname) override
		{
			return Inner.ShouldVisitLeafPathname(LeafPathname);
		}
		virtual bool Visit(const FString& Filename, bool bIsDirectory, const FString* AbsPath) override
		{
			FFileStatData StatData;
			if (!bIsDirectory && AbsPath)
			{
				StatData = LowerLevel.GetStatData(**AbsPath);
			}
			else if (bIsDirectory)
			{
				StatData.bIsValid = true;
				StatData.bIsDirectory = true;
				StatData.bIsReadOnly = true;
				StatData.FileSize = -1;
			}
			return Inner.Visit(*Filename, StatData);
		}

		IPlatformFile& LowerLevel;
		IPlatformFile::FDirectoryStatVisitor& Inner;
	};

}


class FPlatformDLCPlatformFile : public IPlatformDLCPlatformFile
{
public:
	FPlatformDLCPlatformFile() = default;
	virtual ~FPlatformDLCPlatformFile() = default;

	virtual bool Initialize(IPlatformFile* Inner, const TCHAR* CmdLine) override
	{
		LowerLevel = Inner;
		return true;
	}

	virtual IPlatformFile* GetLowerLevel() override
	{
		return LowerLevel;
	}

	virtual void SetLowerLevel(IPlatformFile* NewLowerLevel) override
	{
		LowerLevel = NewLowerLevel;
	}

	virtual const TCHAR* GetName() const override
	{
		return TEXT("PlatformDLC");
	}

	virtual bool FileExists(const TCHAR* Filename) override
	{
		if (LowerLevel->FileExists(Filename))
		{
			return true;
		}

		FString AbsPath;
		return FindFileInDLC(Filename, AbsPath);
	}

	virtual int64 FileSize(const TCHAR* Filename) override
	{
		if (LowerLevel->FileExists(Filename))
		{
			return LowerLevel->FileSize(Filename);
		}

		FString AbsPath;
		if (FindFileInDLC(Filename, AbsPath))
		{
			return LowerLevel->FileSize(*AbsPath);
		}
		return -1;
	}

	virtual bool DeleteFile(const TCHAR* Filename) override
	{
		return LowerLevel->DeleteFile(Filename);
	}

	virtual bool IsReadOnly(const TCHAR* Filename) override
	{
		if (LowerLevel->FileExists(Filename))
		{
			return LowerLevel->IsReadOnly(Filename);
		}

		FString AbsPath;
		return FindFileInDLC(Filename, AbsPath); // DLC files are always read-only
	}

	virtual bool MoveFile(const TCHAR* To, const TCHAR* From) override
	{
		return LowerLevel->MoveFile(To, From);
	}

	virtual bool SetReadOnly(const TCHAR* Filename, bool bNewReadOnlyValue) override
	{
		return LowerLevel->SetReadOnly(Filename, bNewReadOnlyValue);
	}

	virtual FDateTime GetTimeStamp(const TCHAR* Filename) override
	{
		if (LowerLevel->FileExists(Filename))
		{
			return LowerLevel->GetTimeStamp(Filename);
		}

		FString AbsPath;
		if (FindFileInDLC(Filename, AbsPath))
		{
			return LowerLevel->GetTimeStamp(*AbsPath);
		}
		return FDateTime::MinValue();
	}

	virtual void SetTimeStamp(const TCHAR* Filename, FDateTime DateTime) override
	{
		LowerLevel->SetTimeStamp(Filename, DateTime);
	}

	virtual FDateTime GetAccessTimeStamp(const TCHAR* Filename) override
	{
		if (LowerLevel->FileExists(Filename))
		{
			return LowerLevel->GetAccessTimeStamp(Filename);
		}

		FString AbsPath;
		if (FindFileInDLC(Filename, AbsPath))
		{
			return LowerLevel->GetAccessTimeStamp(*AbsPath);
		}
		return FDateTime::MinValue();
	}

	virtual FString GetFilenameOnDisk(const TCHAR* Filename) override
	{
		if (LowerLevel->FileExists(Filename))
		{
			return LowerLevel->GetFilenameOnDisk(Filename);
		}

		FString AbsPath;
		if (FindFileInDLC(Filename, AbsPath))
		{
			return LowerLevel->GetFilenameOnDisk(*AbsPath);
		}
		return Filename;
	}

	virtual IFileHandle* OpenRead(const TCHAR* Filename, bool bAllowWrite = false) override
	{
		IFileHandle* Handle = LowerLevel->OpenRead(Filename, bAllowWrite);
		if (Handle)
		{
			return Handle;
		}

		FString AbsPath;
		if (FindFileInDLC(Filename, AbsPath))
		{
			return LowerLevel->OpenRead(*AbsPath, bAllowWrite);
		}
		return nullptr;
	}

	virtual IFileHandle* OpenWrite(const TCHAR* Filename, bool bAppend = false, bool bAllowRead = false) override
	{
		return LowerLevel->OpenWrite(Filename, bAppend, bAllowRead);
	}

	virtual FFileStatData GetStatData(const TCHAR* FilenameOrDirectory) override
	{
		FFileStatData StatData = LowerLevel->GetStatData(FilenameOrDirectory);
		if (StatData.bIsValid)
		{
			return StatData;
		}

		FString AbsPath;
		if (FindFileInDLC(FilenameOrDirectory, AbsPath))
		{
			return LowerLevel->GetStatData(*AbsPath);
		}
		if (IsDLCDirectory(FilenameOrDirectory))
		{
			FFileStatData DirStat;
			DirStat.bIsValid = true;
			DirStat.bIsDirectory = true;
			DirStat.bIsReadOnly = true;
			DirStat.FileSize = -1;
			return DirStat;
		}
		return StatData;
	}

	virtual bool DirectoryExists(const TCHAR* Directory) override
	{
		if (LowerLevel->DirectoryExists(Directory))
		{
			return true;
		}

		return IsDLCDirectory(Directory);
	}

	virtual bool CreateDirectory(const TCHAR* Directory) override
	{
		return LowerLevel->CreateDirectory(Directory);
	}

	virtual bool DeleteDirectory(const TCHAR* Directory) override
	{
		return LowerLevel->DeleteDirectory(Directory);
	}

	virtual bool IterateDirectory(const TCHAR* Directory, FDirectoryVisitor& Visitor) override
	{
		bool bResult = LowerLevel->IterateDirectory(Directory, Visitor);

		UE::PlatformDLC::Private::FDLCFileDirectoryVisitor DLCVisitor(Visitor);
		return IterateDirectoryInDLCFiles(Directory, DLCVisitor) || bResult;
	}

	virtual bool IterateDirectoryStat(const TCHAR* Directory, FDirectoryStatVisitor& Visitor) override
	{
		bool bResult = LowerLevel->IterateDirectoryStat(Directory, Visitor);

		UE::PlatformDLC::Private::FDLCFileDirectoryStatVisitor DLCStatVisitor(*LowerLevel, Visitor);
		return IterateDirectoryInDLCFiles(Directory, DLCStatVisitor) || bResult;
	}

	//~ For visibility of overloads we don't override
	using IPlatformFile::IterateDirectory;
	using IPlatformFile::IterateDirectoryRecursively;
	using IPlatformFile::IterateDirectoryStat;
	using IPlatformFile::IterateDirectoryStatRecursively;
	using IPlatformFile::OpenAsyncRead;

	virtual bool DeleteDirectoryRecursively(const TCHAR* Directory) override
	{
		return LowerLevel->DeleteDirectoryRecursively(Directory);
	}

	virtual bool CopyFile(const TCHAR* To, const TCHAR* From, EPlatformFileRead ReadFlags = EPlatformFileRead::None, EPlatformFileWrite WriteFlags = EPlatformFileWrite::None) override
	{
		return LowerLevel->CopyFile(To, From, ReadFlags, WriteFlags);
	}

	virtual bool CreateDirectoryTree(const TCHAR* Directory) override
	{
		return LowerLevel->CreateDirectoryTree(Directory);
	}

	virtual bool CopyDirectoryTree(const TCHAR* DestinationDirectory, const TCHAR* Source, bool bOverwriteAllExisting) override
	{
		return LowerLevel->CopyDirectoryTree(DestinationDirectory, Source, bOverwriteAllExisting);
	}

	virtual FString ConvertToAbsolutePathForExternalAppForRead(const TCHAR* Filename) override
	{
		if (LowerLevel->FileExists(Filename))
		{
			return LowerLevel->ConvertToAbsolutePathForExternalAppForRead(Filename);
		}

		FString AbsPath;
		if (FindFileInDLC(Filename, AbsPath))
		{
			return AbsPath;
		}

		return LowerLevel->ConvertToAbsolutePathForExternalAppForRead(Filename);
	}

	virtual FString ConvertToAbsolutePathForExternalAppForWrite(const TCHAR* Filename) override
	{
		return LowerLevel->ConvertToAbsolutePathForExternalAppForWrite(Filename);
	}

	virtual IAsyncReadFileHandle* OpenAsyncRead(const TCHAR* Filename, bool bAllowWrite = false) override
	{
		if (!LowerLevel->FileExists(Filename))
		{
			FString AbsPath;
			if (FindFileInDLC(Filename, AbsPath))
			{
				return LowerLevel->OpenAsyncRead(*AbsPath, bAllowWrite);
			}
		}
		return LowerLevel->OpenAsyncRead(Filename, bAllowWrite);
	}

	virtual FOpenMappedResult OpenMappedEx(const TCHAR* Filename, EOpenReadFlags OpenOptions = EOpenReadFlags::None, int64 MaximumSize = 0) override
	{
		FOpenMappedResult Result = LowerLevel->OpenMappedEx(Filename, OpenOptions, MaximumSize);
		if (Result.HasValue())
		{
			return Result;
		}

		FString AbsPath;
		if (FindFileInDLC(Filename, AbsPath))
		{
			return LowerLevel->OpenMappedEx(*AbsPath, OpenOptions, MaximumSize);
		}
		return Result;
	}

	virtual bool SendMessageToServer(const TCHAR* Message, IFileServerMessageHandler* Handler) override
	{
		return LowerLevel->SendMessageToServer(Message, Handler);
	}



protected:
	virtual void AddMountPoint( FName DLCName, const FString& MountPoint ) override
	{
		// early out if we have this already
		{
			FReadScopeLock Lock(MountPointsLock);
			if (MountPoints.Contains(DLCName))
			{
				return;
			}
		}

		TSharedPtr<FDLCFiles> DLCFiles = MakeShared<FDLCFiles>();
		DLCFiles->MountPoint = MountPoint;

		// DLC root directory is often an unusual path outside of the game's root
		FString NormalizedDLCRoot = MountPoint;
		FPaths::NormalizeDirectoryName(NormalizedDLCRoot);
		NormalizedDLCRoot += TEXT("/"); // MakePathRelativeTo expects a trailing / on the path

		// enumerate all the files in the DLC, ignoring anything in the pak folder
		TArray<FString> FoundFiles;
		IPlatformFile::GetPlatformPhysical().FindFilesRecursively(FoundFiles, *MountPoint, TEXT(""));
		for (FString& File : FoundFiles)
		{
			FString Directory = FPaths::GetPath(File);
			if (Directory.EndsWith(TEXT("/Paks"), ESearchCase::IgnoreCase))
			{
				continue;
			}

			FString RelativePath = File;
			FPaths::MakePathRelativeTo(RelativePath, *NormalizedDLCRoot);

			DLCFiles->Files.FindOrAdd(RelativePath) = true;
		}
		DLCFiles->Files.Shrink();

		// save this DLC if we found loose files
		if (!DLCFiles->Files.IsEmpty())
		{
			UE_LOGF(LogPlatformDLC, Log, "DLC %ls loose file tree is %llu bytes for %d files", *DLCName.ToString(), DLCFiles->Files.GetAllocatedSize(), DLCFiles->Files.Num());

			FWriteScopeLock Lock(MountPointsLock);
			MountPoints.Add(DLCName, MoveTemp(DLCFiles));
		}

#if PLATFORMDLC_DUMP_LOOSE_FILES_ON_MOUNT
		{
			FReadScopeLock Lock(MountPointsLock);
			UE_LOGF(LogPlatformDLC, Log, "%d mounted DLCs. Unified loose files are:", MountPoints.Num() );
		}
		DumpLooseFiles();
#endif
	}

	virtual void RemoveMountPoint( FName DLCName ) override
	{
		FWriteScopeLock Lock(MountPointsLock);
		MountPoints.Remove(DLCName);
	}

	virtual bool HasMountPoints() const override
	{
		FReadScopeLock Lock(MountPointsLock);
		return MountPoints.Num() > 0;
	}

private:
	struct FDLCFiles
	{
		FString MountPoint;
		TDirectoryTree<bool> Files; // relative to game root
	};

	bool IterateDirectoryInDLCFiles(const TCHAR* Directory, UE::PlatformDLC::Private::FDLCFileDirectoryVisitorBase& Visitor)
	{
		bool bResult = false;

		// early out for unsupported paths
		if (!FStringView(Directory).StartsWith(TEXTVIEW("../../../")))
		{
			return bResult;
		}
		FString RelDir = MakeRelativeToDLCRoot(Directory);

		// enumerate paths from all mounted DLC
		TSet<FString> ChildPaths;
		{
			FReadScopeLock Lock(MountPointsLock);
			for (const TPair<FName, TSharedPtr<FDLCFiles>>& Pair : MountPoints)
			{
				const FDLCFiles& DLCFiles = *Pair.Value;
				TArray<FString> ChildNames;
				if (!DLCFiles.Files.TryGetChildren(RelDir, ChildNames, EDirectoryTreeGetFlags::ImpliedParent | EDirectoryTreeGetFlags::ImpliedChildren))
				{
					continue;
				}

				bResult = true;
				for (const FString& ChildName : ChildNames)
				{
					if (!Visitor.ShouldVisitLeafPathname(ChildName))
					{
						continue;
					}

					FString ChildPath = FPaths::Combine(Directory, *ChildName);
					FString ChildRelPath = FPaths::Combine(RelDir, ChildName);

					if (DLCFiles.Files.ContainsChildPaths(ChildRelPath))
					{
						ChildPaths.Add(ChildPath);
					}
					else
					{
						FString AbsChildPath = FPaths::Combine(DLCFiles.MountPoint, ChildRelPath);
						if (!Visitor.Visit(ChildPath, false, &AbsChildPath))
						{
							return false;
						}
					}
				}
			}
		}

		// must iterate paths separately because the same directory is likely to exist in multiple DLCs (../../../MyGame/Content/ for example)
		for (const FString& ChildPath : ChildPaths)
		{
			if (!Visitor.Visit(ChildPath, true, nullptr))
			{
				return false;
			}
		}

		return bResult;
	}

	static FString MakeRelativeToDLCRoot(const TCHAR* InPath)
	{
		FString Path = InPath;
		if (!Path.RemoveFromStart(TEXT("../../../")))
		{
			return FString();
		}
		return Path;
	}

	bool FindFileInDLC(const TCHAR* InPath, FString& OutAbsPath) const
	{
		FString RelPath = MakeRelativeToDLCRoot(InPath);
		if (RelPath.IsEmpty())
		{
			return false;
		}

		FReadScopeLock Lock(MountPointsLock);
		for (const TPair<FName, TSharedPtr<FDLCFiles>>& Pair : MountPoints)
		{
			if (Pair.Value->Files.Contains(RelPath))
			{
				OutAbsPath = FPaths::Combine(Pair.Value->MountPoint, RelPath);
				return true;
			}
		}
		return false;
	}

	bool IsDLCDirectory(const TCHAR* InPath) const
	{
		FString RelPath = MakeRelativeToDLCRoot(InPath);
		if (RelPath.IsEmpty())
		{
			return false;
		}

		FReadScopeLock Lock(MountPointsLock);
		for (const TPair<FName, TSharedPtr<FDLCFiles>>& Pair : MountPoints)
		{
			if (Pair.Value->Files.ContainsChildPaths(RelPath))
			{
				return true;
			}
		}
		return false;
	}


#if PLATFORMDLC_DUMP_LOOSE_FILES_ON_MOUNT
	void DumpLooseFiles(const TCHAR* Directory = TEXT("../../../"))
	{
		using namespace UE::PlatformDLC::Private;

		class FDumpVisitor : public IPlatformFile::FDirectoryVisitor
		{
		public:
			TArray<FString> Subdirectories;

			virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
			{
				if (bIsDirectory)
				{
					Subdirectories.Add(FilenameOrDirectory);
				}
				else
				{
					UE_LOGF(LogPlatformDLC, Log, "  %ls", FilenameOrDirectory);
				}
				return true;
			}
		};

		FDumpVisitor InnerVisitor;
		FDLCFileDirectoryVisitor DLCVisitor(InnerVisitor);
		IterateDirectoryInDLCFiles(Directory, DLCVisitor);

		for (const FString& Subdir : InnerVisitor.Subdirectories)
		{
			DumpLooseFiles(*Subdir);
		}
	}
#endif



	IPlatformFile* LowerLevel = nullptr;

	mutable FRWLock MountPointsLock;
	TMap<FName, TSharedPtr<FDLCFiles>> MountPoints;
};




TUniquePtr<IPlatformDLCPlatformFile> IPlatformDLCPlatformFile::Construct()
{
	return MakeUnique<FPlatformDLCPlatformFile>();
}

