// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GenericPlatform/GenericPlatformFile.h"

class IPlatformDLCPlatformFile : public IPlatformFile
{
public:
	static TUniquePtr<IPlatformDLCPlatformFile> Construct();

	virtual ~IPlatformDLCPlatformFile() = default;

	// IPlatformDLCPlatformFile interface
	virtual void AddMountPoint( FName DLCName, const FString& MountPoint ) = 0;
	virtual void RemoveMountPoint( FName DLCName ) = 0;
	virtual bool HasMountPoints() const = 0;
};
