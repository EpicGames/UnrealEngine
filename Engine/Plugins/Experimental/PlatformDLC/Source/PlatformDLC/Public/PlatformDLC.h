// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Modules/ModuleInterface.h"
#include <atomic>

#define UE_API PLATFORMDLC_API

// main interface for Platform DLC
// note: API may be subject to change while this plugin is experimental
class IPlatformDLC
{
public:
	virtual ~IPlatformDLC() = default;

	enum class ENotification
	{
		Entitlement, // true = entitlement gained, false = lost
		Mounted,     // whether the DLC has been successfully mounted and can now be used
		Unmounted,   // whether the DLC has been successfully unmounted
		Downloaded,  // whether the DLC has been successfully downloaded and can now be mounted (note if AutoMount is configured, this will happen automatically)
		Uninstalled, // whether the DLC has been successfully uninstalled
	};
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnDLCNotification, FName, ENotification, bool/*success*/ )

	enum class EState : uint8
	{
		NotInstalled,
		Downloading,
		Downloaded,
		Mounting,
		Mounted,
		Unmounting,
		Uninstalling,
	};

	// start initializing the platform DLC; subsequent calls after the first are ignored; returns true if initialization was started
	virtual bool InitializeAsync() = 0;

	// register a callback for when initialization is complete; the callback will still occur if the DLC is already initialized
	virtual void RegisterInitializationCallback(TFunction<void()> Callback) = 0;

	// shutdown the platform DLC and clean up resources
	virtual void Shutdown() = 0;

	// determines whether the platform DLC has finished initializing
	virtual bool IsInitialized() const = 0;


	// returns true if DLC is supported in the current build configuration. Some platforms may only support DLC based
	// on how its run (e.g., packaged builds) or it may be preferred in development to disable DLC.
	virtual bool IsAvailable() = 0;


	// query if we have entitlement to download & mount the given DLC
	virtual bool HasEntitlement( FName DLCName ) const = 0;

	// returns the current state of the given DLC
	virtual EState GetState( FName DLCName ) const = 0;

	// called when there is a notification about DLC, such as mounting completion
	virtual FOnDLCNotification& OnNotification() = 0;


	// start asynchronously mounting the given DLC
	// can also be used to attempt entitlement reacquisition after it has been lost
	// returns true if the callback will be triggered, or false if the DLC is not known
	virtual bool Mount( FName DLCName ) = 0;

	// start asynchronously unmounting the given DLC. It is essential that the game is no longer using the DLC
	// returns true if the callback will be triggered, or false if the DLC is not known
	virtual bool Unmount( FName DLCName ) = 0;

	// start asynchronously downloading the given DLC. Depending on the platform you may reacquire entitlements before you can download.
	// returns true if the callback will be triggered, or false if the DLC is not known
	virtual bool Download( FName DLCName ) = 0;

	// start uninstalling the given DLC (may not work properly on all platforms. DLC must be unmounted first)
	// returns true if the callback will be triggered, or false if the DLC is not known
	virtual bool Uninstall( FName DLCName ) = 0;


	// determine the download size for the given DLC
	// returns true if the download size is available
	virtual bool GetDownloadSize( FName DLCName, uint64& OutCurrentInstallSize, uint64& OutFullInstallSize ) const = 0;

	// returns the file system mount point for the given DLC
	virtual FString GetRootDirectory( FName DLCName ) const = 0;

	// returns the list of all known DLC (regardless of whether downloaded / mounted)
	virtual TArray<FName> GetAllDLCNames() const = 0;

	// returns the list of all DLC that is ready for use (entitled, downloaded & mounted)
	virtual TArray<FName> GetMountedDLCNames() const = 0;

	// get the store id for the DLC (should be convertible to FUniqueOfferId)
	virtual FString GetStoreId( FName DLCName ) const = 0;

	// set the user who will be used to purchase any DLC
	virtual void SetStoreUser( FPlatformUserId PlatformUserId ) = 0;

	// returns the user who will be used to purchase any DLC, if any
	virtual FPlatformUserId GetStoreUser() const = 0;



	// config section for Platform DLC settings
	static UE_API const TCHAR* ConfigSectionName;
};


// interface for a platform DLC factory
class IPlatformDLCFactoryModule : public IModuleInterface
{
public:
	// get the platform DLC for this factory
	virtual TSharedPtr<IPlatformDLC> GetPlatformDLC() = 0;
};


// generic platform DLC implementation - typical base class for platform specializations
class FGenericPlatformDLC : public IPlatformDLC
{
public:
	UE_API virtual bool InitializeAsync() override final;
	UE_API virtual void RegisterInitializationCallback(TFunction<void()> Callback) override final;
	UE_API virtual void Shutdown() override;

	virtual bool IsInitialized() const override
	{
		return InitState.load(std::memory_order_acquire) == EInitState::Complete;
	}

	virtual bool IsAvailable() override
	{
		return true;
	}

	virtual bool HasEntitlement( FName DLCName ) const override
	{
		return false;
	}

	virtual EState GetState( FName DLCName ) const override
	{
		return EState::NotInstalled;
	}

	virtual FOnDLCNotification& OnNotification() override
	{
		return NotificationDelegate;
	}

	virtual bool Mount( FName DLCName ) override
	{
		return false;
	}

	virtual bool Unmount( FName DLCName ) override
	{
		return false;
	}

	virtual bool Download( FName DLCName ) override
	{
		return false;
	}

	virtual bool Uninstall( FName DLCName ) override
	{
		return false;
	}

	virtual bool GetDownloadSize( FName DLCName, uint64& OutCurrentInstallSize, uint64& OutFullInstallSize ) const override
	{
		return false;
	}

	virtual FString GetRootDirectory( FName DLCName ) const override
	{
		return FString();
	}

	virtual TArray<FName> GetAllDLCNames() const override
	{
		return TArray<FName>();
	}

	virtual TArray<FName> GetMountedDLCNames() const override
	{
		return TArray<FName>();
	}

	virtual FString GetStoreId( FName DLCName ) const override
	{
		return FString();
	}

	virtual void SetStoreUser( FPlatformUserId PlatformUserId ) override
	{
	}

	virtual FPlatformUserId GetStoreUser() const override
	{
		return PLATFORMUSERID_NONE;
	}

protected:
	// override to perform platform-specific initialization on any thread and call EndInitializeInternal() when finished
	virtual void BeginInitializeInternal() {}

	UE_API void EndInitializeInternal();

	bool IsInitializing() const
	{
		return InitState.load(std::memory_order_acquire) == EInitState::InProgress;
	}

	UE_API void PostNotification( FName DLCName, ENotification Notification, bool bSuccess );

private:
	enum class EInitState : uint8
	{
		NotStarted, 
		InProgress, 
		Complete 
	};

	mutable FCriticalSection      InitLock;
	TArray<TFunction<void()>>     PendingInitCallbacks;
	std::atomic<EInitState>       InitState{ EInitState::NotStarted };

	FOnDLCNotification NotificationDelegate;
};



UE_API FString LexToString(IPlatformDLC::ENotification Notification);
UE_API FString LexToString(IPlatformDLC::EState State);



#undef UE_API
