// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PlatformDLC.h"
#include "HAL/IPlatformFileModule.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"

PLATFORMDLC_API DECLARE_LOG_CATEGORY_EXTERN(LogPlatformDLC, Log, All)

class IPlatformDLCModule : public IPlatformFileModule
{
public:
	/**
	 * Singleton-like access to the platform DLC module instance
	 * @return Returns IPlatformDLCModule singleton instance, loading the module on demand if needed
	 */
	static inline IPlatformDLCModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IPlatformDLCModule>("PlatformDLC");
	}

	/**
	 * Returns the platform DLC, if any (configured via ini)
	 * @return platform DLC pointer, or null if it isn't available
	 */
	virtual TSharedPtr<IPlatformDLC> GetPlatformDLC() = 0;
};
